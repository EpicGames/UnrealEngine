{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "NiagaraFluids",
	"Description": "Fluid simulation toolkit for Niagara",
	"Category": "FX",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "NiagaraFluids",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],    
	"Plugins": [
        {
            "Name": "Niagara",			
            "Enabled": true
        }
	]
}