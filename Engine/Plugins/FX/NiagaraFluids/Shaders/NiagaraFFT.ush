// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NiagaraFFT.ush
=============================================================================*/

// Shared memory for FFT
groupshared float2 FFTGroupShared[2*THREADGROUP_SIZE];

// common FFT functions
// complex conjugate
float2 ComplexCon(float2 Z) 
{
	return float2(Z.x, -Z.y);
}

float2 ComplexMultEqs(float2 ZA, float2 ZB)
{ 
	float2 _Tmp = ZA;
	return float2(ZA.x * ZB.x - ZA.y * ZB.y, ZA.x * ZB.y + ZA.y * ZB.x);
}

// Laplacian [ f_{i+1, j} + f_{i-1,1} + f_{i,j+1} + f_{i,j-1} - 4 ] / (dx * dx)
//
// for the mode f_{nx, ny) =  Sin(Pi kx nx / M) Sin (Pi ky ny / M)
//  
// the eignevalue is  -(4/ (dx*dx) ) ( Sin^2(Pi/2  kx/M ) + Sin^2(Pi/2 ky/M) )
//     can also write this as
//                    (2/dx*dx) ( Cos(Pi kx/M ) + Cos(Pi ky/M) - 2)

float EigenValue(int kx_, int ky_, int Mx, int My, float Pi, float dx)
{
	return ( -4 * (pow( sin( Pi * float(kx_) / float(2 * Mx) )/dx, 2) +  pow( sin( Pi * float(ky_) / float(2 * My) )/dx, 2)));
}