// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
NiagaraDataInterfacePhysicsField.ush 
=============================================================================*/

#pragma once

/* -----------------------------------------------------------------
 * Rendering code for simulation stages
 * -----------------------------------------------------------------
 */
#define MARCH_SINGLE_RAY(RayStart, RayDir, UnitStepSize, LocalStepSize, NumSteps, Volume, RetV) \
RetV = float4(0,0,0,0); \
float Transmittance = 1; \
int Steps = ceil(NumSteps); \
if(Steps >= 1.0 ) Steps += 1.0; \
[loop] \
for (int i = 0; i < Steps && Transmittance > 1e-5; i++) \
{ \
float Extinction; \
float Scattering; \
float3 Luminance; \
Volume.ComputeStep(RayStart, RayDir, Extinction, Scattering, Luminance); \
Luminance *= Scattering; \
float ClampedExtinction = max(Extinction, 1e-6); \
float StepTransmittance = exp(-Extinction * LocalStepSize); \
RetV.rgb += Transmittance * (Luminance - Luminance * StepTransmittance) / ClampedExtinction; \
Transmittance *= StepTransmittance; \
if(i == Steps-2) UnitStepSize, LocalStepSize *= frac(NumSteps); \
RayStart += RayDir * UnitStepSize; \
} \
RetV.a = 1. - Transmittance;

#define MARCH_SINGLE_RAY_EMISSION(RayStart, RayDir, UnitStepSize, LocalStepSize, NumSteps, Volume, RetV) \
RetV = float4(0,0,0,0); \
float Transmittance = 1; \
int Steps = ceil(NumSteps); \
if(Steps >= 1.0 ) Steps += 1.0; \
[loop] \
for (int i = 0; i < Steps && Transmittance > 1e-5; i++) \
{ \
float Extinction; \
float Scattering; \
float3 Luminance; \
float EmissionExtinction = 0; \
float3 EmissionColor = float3(0,0,0); \
Volume.ComputeStep(RayStart, RayDir, Extinction, Scattering, Luminance, EmissionExtinction, EmissionColor); \
float ClampedEmissionExtinction = max(EmissionExtinction, 1e-6); \
float StepEmissionTransmittance = exp(-EmissionExtinction * LocalStepSize); \
RetV.rgb += Transmittance * (EmissionColor - EmissionColor * StepEmissionTransmittance) / ClampedEmissionExtinction; \
Transmittance *= StepEmissionTransmittance; \
Luminance *= Scattering; \
float ClampedExtinction = max(Extinction, 1e-6); \
float StepTransmittance = exp(-Extinction * LocalStepSize); \
RetV.rgb += Transmittance * (Luminance - Luminance * StepTransmittance) / ClampedExtinction; \
Transmittance *= StepTransmittance; \
if(i == Steps-2) UnitStepSize, LocalStepSize *= frac(NumSteps); \
RayStart += RayDir * UnitStepSize; \
} \
RetV.a = 1. - Transmittance;
