// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class NiagaraFluids : ModuleRules
	{
		public NiagaraFluids(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.Add("Core");
			PrivateDependencyModuleNames.Add("RenderCore");
			PrivateDependencyModuleNames.Add("Projects");
		}
	}
}
