// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "INiagaraFluids.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "ShaderCore.h"


class FNiagaraFluids : public INiagaraFluids
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FNiagaraFluids, NiagaraFluids )


void FNiagaraFluids::StartupModule()
{
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("NiagaraFluids"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/FX/NiagaraFluids"), PluginShaderDir);
}


void FNiagaraFluids::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}
